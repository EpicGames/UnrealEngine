// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/Array.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "Dialogs/Dialogs.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "ISettingsModule.h"
#include "Interfaces/IProjectManager.h"
#include "Interfaces/IProjectTargetPlatformEditorModule.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Modules/ModuleManager.h"
#include "PlatformInfo.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SProjectTargetPlatformSettings.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;


#define LOCTEXT_NAMESPACE "FProjectTargetPlatformEditorModule"


/**
 * Implements the platform target platform editor module
 */
class FProjectTargetPlatformEditorModule
	: public IProjectTargetPlatformEditorModule
{
public:

	// IProjectTargetPlatformEditorModule interface

	virtual TWeakPtr<SWidget> CreateProjectTargetPlatformEditorPanel() override
	{
		TSharedPtr<SWidget> Panel = SNew(SProjectTargetPlatformSettings);
		EditorPanels.Add(Panel);

		return Panel;
	}

	virtual void DestroyProjectTargetPlatformEditorPanel(const TWeakPtr<SWidget>& Panel) override
	{
		EditorPanels.Remove(Panel.Pin());
	}

	virtual void AddOpenProjectTargetPlatformEditorMenuItem(FMenuBuilder& MenuBuilder) const override
	{
		struct Local
		{
			static void OpenSettings( FName ContainerName, FName CategoryName, FName SectionName )
			{
				FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer(ContainerName, CategoryName, SectionName);
			}
		};

		MenuBuilder.AddMenuEntry(
			LOCTEXT("SupportedPlatformsMenuLabel", "Supported Platforms..."),
			LOCTEXT("SupportedPlatformsMenuToolTip", "Change which platforms this project supports"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateStatic(&Local::OpenSettings, FName("Project"), FName("Project"), FName("SupportedPlatforms")))
			);
	}

	virtual TSharedRef<SWidget> MakePlatformMenuItemWidget(const PlatformInfo::FTargetPlatformInfo& PlatformInfo, const bool bForCheckBox = false, const FText& DisplayNameOverride = FText()) const override
	{
		struct Local
		{
			static EVisibility IsUnsupportedPlatformWarningVisible(const FName PlatformName)
			{
				FProjectStatus ProjectStatus;
				return (!IProjectManager::Get().QueryStatusForCurrentProject(ProjectStatus) || ProjectStatus.IsTargetPlatformSupported(PlatformName)) ? EVisibility::Hidden : EVisibility::Visible;
			}
		};

		const float MenuIconSize = FCoreStyle::Get().GetFloat("Menu.MenuIconSize", nullptr, 16.f);

		return 
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin((bForCheckBox) ? 2 : 13, 0, 2, 0))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SOverlay)
				+SOverlay::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.WidthOverride(MenuIconSize)
					.HeightOverride(MenuIconSize)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush(PlatformInfo.GetIconStyleName(EPlatformIconSize::Normal)))
					]
				]
				+SOverlay::Slot()
				.Padding(FMargin(MenuIconSize * 0.5f, 0, 0, 0))
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Bottom)
				[
					SNew(SBox)
					.WidthOverride(MenuIconSize)
					.HeightOverride(MenuIconSize)
					[
						SNew(SImage)
						.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateStatic(&Local::IsUnsupportedPlatformWarningVisible, PlatformInfo.VanillaInfo->Name)))
						.Image(FAppStyle::GetBrush("Launcher.Platform.Warning"))
					]
				]
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(FMargin((bForCheckBox) ? 2 : 7, 0, 6, 0))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "Menu.Label")
				.Text((DisplayNameOverride.IsEmpty()) ? PlatformInfo.DisplayName : DisplayNameOverride)
			];
	}

	virtual bool ShowUnsupportedTargetWarning(const FName PlatformName) const override
	{
		const PlatformInfo::FTargetPlatformInfo* const PlatformInfo = PlatformInfo::FindPlatformInfo(PlatformName);
		check(PlatformInfo);

		// Don't show the warning during automation testing; the dlg is modal and blocks
		FProjectStatus ProjectStatus;
		if(!GIsAutomationTesting && IProjectManager::Get().QueryStatusForCurrentProject(ProjectStatus) && !ProjectStatus.IsTargetPlatformSupported(PlatformInfo->VanillaInfo->Name))
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("DisplayName"), PlatformInfo->DisplayName);
			FText WarningText = FText::Format(LOCTEXT("ShowUnsupportedPlatformWarning_Message", "{DisplayName} is not listed as a supported platform for this project, so may not run as expected.\n\nDo you wish to continue?"), Args);

			FSuppressableWarningDialog::FSetupInfo Info(
				WarningText, 
				LOCTEXT("ShowUnsupportedPlatformWarning_Title", "Unsupported Platform"), 
				TEXT("SuppressUnsupportedPlatformWarningDialog")
				);
			Info.ConfirmText = LOCTEXT("ShowUnsupportedPlatformWarning_Confirm", "Continue");
			Info.CancelText = LOCTEXT("ShowUnsupportedPlatformWarning_Cancel", "Cancel");
			FSuppressableWarningDialog UnsupportedPlatformWarningDialog(Info);

			return UnsupportedPlatformWarningDialog.ShowModal() != FSuppressableWarningDialog::EResult::Cancel;
		}

		return true;
	}

private:

	// Holds the collection of created editor panels.
	TArray<TSharedPtr<SWidget> > EditorPanels;
};


#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FProjectTargetPlatformEditorModule, ProjectTargetPlatformEditor);
