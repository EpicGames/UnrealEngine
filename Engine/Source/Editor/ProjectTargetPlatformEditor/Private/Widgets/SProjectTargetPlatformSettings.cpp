// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SProjectTargetPlatformSettings.h"

#include "GameProjectGenerationModule.h"
#include "Interfaces/IProjectManager.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateTypes.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE "SProjectTargetPlatformSettings"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SProjectTargetPlatformSettings::Construct(const FArguments& InArgs)
{
	TSharedRef<SVerticalBox> PlatformsListBox = SNew(SVerticalBox);

	for (const FDataDrivenPlatformInfo* DDPI : FDataDrivenPlatformInfoRegistry::GetSortedPlatformInfos(EPlatformInfoType::TruePlatformsOnly))
	{
		PlatformsListBox->AddSlot()
			.AutoHeight()
			[
				MakePlatformRow(FText::FromName(DDPI->IniPlatformName), DDPI->IniPlatformName, FAppStyle::GetBrush(DDPI->GetIconStyleName(EPlatformIconSize::Normal)))
			];
	}

	ChildSlot
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(5.0f)
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				[
					MakePlatformRow(LOCTEXT("AllPlatforms", "All Platforms"), NAME_None, FAppStyle::GetBrush("Launcher.Platform.AllPlatforms"))
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(0.0f, 1.0f))
				[
					SNew(SSeparator)
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				[
					PlatformsListBox
				]
			]
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(0.0f, 5.0f))
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(5.0f)
			[
				SNew(STextBlock)
				.AutoWrapText(true)
				.Text(LOCTEXT("PlatformsListDescription", "Select the supported platforms for your project. Attempting to package, run, or cook your project on an unsupported platform will result in a warning."))
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedRef<SWidget> SProjectTargetPlatformSettings::MakePlatformRow(const FText& DisplayName, const FName PlatformName, const FSlateBrush* Icon) const
{
	return SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(5.0f, 0.0f))
		[
			SNew(SCheckBox)
			.IsChecked(this, &SProjectTargetPlatformSettings::HandlePlatformCheckBoxIsChecked, PlatformName)
			.IsEnabled(this, &SProjectTargetPlatformSettings::HandlePlatformCheckBoxIsEnabled, PlatformName)
			.OnCheckStateChanged(this, &SProjectTargetPlatformSettings::HandlePlatformCheckBoxStateChanged, PlatformName)
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(5.0f, 2.0f))
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.WidthOverride(20.f)
			.HeightOverride(20.f)
			[
				SNew(SImage)
				.Image(Icon)
			]
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(5.0f, 0.0f))
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(DisplayName)
		];
}

ECheckBoxState SProjectTargetPlatformSettings::HandlePlatformCheckBoxIsChecked(const FName PlatformName) const
{
	FProjectStatus ProjectStatus;
	if(IProjectManager::Get().QueryStatusForCurrentProject(ProjectStatus))
	{
		if(PlatformName.IsNone())
		{
			// None is "All Platforms"
			return (ProjectStatus.SupportsAllPlatforms()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
		else
		{
			return (ProjectStatus.IsTargetPlatformSupported(PlatformName)) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
	}

	return ECheckBoxState::Unchecked;
}

bool SProjectTargetPlatformSettings::HandlePlatformCheckBoxIsEnabled(const FName PlatformName) const
{
	FProjectStatus ProjectStatus;
	if(IProjectManager::Get().QueryStatusForCurrentProject(ProjectStatus))
	{
		if(PlatformName.IsNone())
		{
			// None is "All Platforms"
			return true;
		}
		else
		{
			// Individual platforms are only enabled when we're not supporting "All Platforms"
			return !ProjectStatus.SupportsAllPlatforms();
		}
	}

	return false;
}

void SProjectTargetPlatformSettings::HandlePlatformCheckBoxStateChanged(ECheckBoxState InState, const FName PlatformName) const
{
	if(PlatformName.IsNone())
	{
		// None is "All Platforms"
		if(InState == ECheckBoxState::Checked)
		{
			FGameProjectGenerationModule::Get().ClearSupportedTargetPlatforms();
		}
		else
		{
			// We've deselected "All Platforms", so manually select every available platform
			for (const FDataDrivenPlatformInfo* DDPI : FDataDrivenPlatformInfoRegistry::GetSortedPlatformInfos(EPlatformInfoType::TruePlatformsOnly))
			{
				FGameProjectGenerationModule::Get().UpdateSupportedTargetPlatforms(DDPI->IniPlatformName, true);
			}
		}
	}
	else
	{
		FGameProjectGenerationModule::Get().UpdateSupportedTargetPlatforms(PlatformName, InState == ECheckBoxState::Checked);
	}
}

#undef LOCTEXT_NAMESPACE
