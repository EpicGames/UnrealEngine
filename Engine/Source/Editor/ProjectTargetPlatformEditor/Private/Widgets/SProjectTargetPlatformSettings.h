// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FText;
class SWidget;
namespace PlatformInfo { struct FTargetPlatformInfo; }
struct FSlateBrush;

enum class ECheckBoxState : uint8;

class SProjectTargetPlatformSettings : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SProjectTargetPlatformSettings)
	{}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

private:
	/** Generate an entry for the given platform information */
	TSharedRef<SWidget> MakePlatformRow(const FText& DisplayName, const FName PlatformName, const FSlateBrush* Icon) const;

	/** Check to see if the "enabled" checkbox should be checked for this platform */
	ECheckBoxState HandlePlatformCheckBoxIsChecked(const FName PlatformName) const;

	/** Check to see if the "enabled" checkbox should be enabled for this platform */
	bool HandlePlatformCheckBoxIsEnabled(const FName PlatformName) const;

	/** Handle the "enabled" checkbox state being changed for this platform */
	void HandlePlatformCheckBoxStateChanged(ECheckBoxState InState, const FName PlatformName) const;

	TArray<const PlatformInfo::FTargetPlatformInfo*> AvailablePlatforms;
};
