// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class ProjectTargetPlatformEditor : ModuleRules
{
	public ProjectTargetPlatformEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Slate",
                "SlateCore",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"DesktopPlatform",
				"Settings",
				"UnrealEd",
				"Projects",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"GameProjectGeneration",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"GameProjectGeneration",
			}
		);
	}
}
