// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Modules/ModuleInterface.h"

class FMenuBuilder;

namespace PlatformInfo
{
	// Forward declare type from DesktopPlatform rather than add an include dependency to everything using IProjectTargetPlatformEditorModule
	struct FTargetPlatformInfo;
}

/**
 * Interface for the platform target platform editor module
 */
class IProjectTargetPlatformEditorModule : public IModuleInterface
{
public:

	/**
	 * Creates an project target platform editor panel widget.
	 *
	 * @return The created widget.
	 */
	virtual TWeakPtr<SWidget> CreateProjectTargetPlatformEditorPanel() = 0;

	/**
	 * Destroys a previously created editor panel widget.
	 *
	 * @param Panel - The panel to destroy.
	 */
	virtual void DestroyProjectTargetPlatformEditorPanel(const TWeakPtr<SWidget>& Panel) = 0;

	/**
	 * Adds a menu item to open the target platform editor panel in the project settings.
	 *
	 * @param MenuBuilder - The menu builder to add the item to.
	 */
	virtual void AddOpenProjectTargetPlatformEditorMenuItem(FMenuBuilder& MenuBuilder) const = 0;

	/**
	 * Creates the widget to use for a platform entry within a FMenuBuilder.
	 *
	 * @param PlatformInfo - The target platform info to build the widget for
	 * @param bForCheckBox - true if the widget is for a checkbox menu item, false if it's for any other menu item (this affects the padding)
	 * @param DisplayNameOverride - An alternate name to use for the platform, otherwise Platform->DisplayName() will be used
	 */
	virtual TSharedRef<SWidget> MakePlatformMenuItemWidget(const PlatformInfo::FTargetPlatformInfo& PlatformInfo, const bool bForCheckBox = false, const FText& DisplayNameOverride = FText()) const = 0;

	/**
	 * Check to see if the given platform is on the list of supported targets, and show a warning message if it's not, allowing the user to continue or cancel
	 *
	 * @param PlatformName - The name of the platform to test.
	 *
	 * @return true if the platform is supported, or the warning was accepted with "continue".
	 */
	virtual bool ShowUnsupportedTargetWarning(const FName PlatformName) const = 0;

public:

	/**
	 * Virtual destructor.
	 */
	virtual ~IProjectTargetPlatformEditorModule() = default;
};
