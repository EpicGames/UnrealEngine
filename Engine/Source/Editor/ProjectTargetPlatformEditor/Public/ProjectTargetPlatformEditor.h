// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/* Boilerplate
 *****************************************************************************/

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()


/* Dependencies
 *****************************************************************************/

#include "Modules/ModuleManager.h"
#include "SlateBasics.h"


/* Interfaces
 *****************************************************************************/

#include "Interfaces/IProjectTargetPlatformEditorModule.h"
